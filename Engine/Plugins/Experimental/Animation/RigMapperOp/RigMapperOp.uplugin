{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "RigMapper Op",
	"Description": "Experimental Retarget Op for re-mapping curves using RigMapper Definitions",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "RigMapperOp",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "IKRig",
			"Enabled": true
		},
		{
			"Name": "RigMapper",
			"Enabled": true
		}
	]
}