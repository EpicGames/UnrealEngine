// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperOp.h"

#include "Animation/AnimCurveUtils.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimNodeBase.h"
#include "RigMapperDefinition.h"
#include "Retargeter/IKRetargetProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigMapperOp)

static FLinearColor MakeCurveColor(const FName& InCurveName)
{
	// Create a color based on the hash of the name
	FRandomStream Stream(GetTypeHash(InCurveName));
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	return FLinearColor::MakeFromHSV8(Hue, 196, 196);
}

void FRigMapperOpHelper::ProcessAnimSequenceCurves(
	FIKRetargetCurvesOpBase::FCurveData InCurveMetaData,
	FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
	FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData,
	FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues,
	bool bCopyAllSourceCurves) const
{
	// We need to copy the RigMapperProcessor to keep this method const. Revisit this for speed at some point.
	FRigMapperProcessor RigMapperProcessorTemp = RigMapperProcessor;
	TSet<FName> OutputNames;

	if (RigMapperProcessorTemp.IsValid())
	{
		RigMapperProcessorTemp.EvaluateFrames(InCurveMetaData.Names, InCurveFrameValues, OutCurveMetaData.Names, OutCurveFrameValues);
		OutCurveMetaData.Flags.Init(4, OutCurveMetaData.Names.Num()); // a value of 4 means Editable curve
		OutCurveMetaData.Colors.SetNum(OutCurveMetaData.Names.Num());
		for (int32 Curve = 0; Curve < OutCurveMetaData.Names.Num(); ++Curve)
		{
			OutCurveMetaData.Colors[Curve] = MakeCurveColor(OutCurveMetaData.Names[Curve]);
			OutputNames.Add(OutCurveMetaData.Names[Curve]);
		}

		// 'pass-through' any input curves which have not been modified, if bCopyAllSourceCurves is true
		if (bCopyAllSourceCurves)
		{
			TArray<int32> InputCurveIndicesToAdd;
			InputCurveIndicesToAdd.Reserve(InCurveMetaData.Names.Num());
			for (int32 InputCurve = 0; InputCurve < InCurveMetaData.Names.Num(); ++InputCurve)
			{
				if (!OutputNames.Contains(InCurveMetaData.Names[InputCurve]))
				{
					InputCurveIndicesToAdd.Add(InputCurve);
				}
			}

			// now add these curves
			OutCurveMetaData.Names.Reserve(OutCurveMetaData.Names.Num() + InputCurveIndicesToAdd.Num());
			OutCurveMetaData.Flags.Reserve(OutCurveMetaData.Flags.Num() + InputCurveIndicesToAdd.Num());
			OutCurveMetaData.Colors.Reserve(OutCurveMetaData.Colors.Num() + InputCurveIndicesToAdd.Num());
			int32 Frame = 0;
			for (TArray<TOptional<float>>& OutputFrame : OutCurveFrameValues)
			{
				OutputFrame.Reserve(OutputFrame.Num() + InputCurveIndicesToAdd.Num());
				for (int32 CurveIndex : InputCurveIndicesToAdd)
				{
					OutputFrame.Add(InCurveFrameValues[Frame][CurveIndex]);
				}

				Frame++;
			}

			for (int32 CurveIndex : InputCurveIndicesToAdd)
			{
				OutCurveMetaData.Names.Add(InCurveMetaData.Names[CurveIndex]);
				OutCurveMetaData.Flags.Add(InCurveMetaData.Flags[CurveIndex]);
				OutCurveMetaData.Colors.Add(InCurveMetaData.Colors[CurveIndex]);
			}
		}
	}
	else
	{
		// Fallback when the RigMapper processor is not valid, tp avoid empty output curves.
		OutCurveMetaData = MoveTemp(InCurveMetaData);
		OutCurveFrameValues = MoveTemp(InCurveFrameValues);
	}
}

bool FRigMapperOpHelper::CheckReInit(const TArray<URigMapperDefinition*>& InCurrentDefinitions)
{
	bool bReInit = false;
	// Reinit if the definitions are not the same
	if (InCurrentDefinitions.Num() != LoadedDefinitions.Num())
	{
		bReInit = true;
	}
	else
	{
		for (int32 DefinitionIndex = 0; DefinitionIndex < InCurrentDefinitions.Num(); DefinitionIndex++)
		{
			const bool bDefinitionEdittedAndNotValidated = InCurrentDefinitions[DefinitionIndex] != nullptr && !InCurrentDefinitions[DefinitionIndex]->WasDefinitionValidated();
			const bool bDefinitionHasChanged = InCurrentDefinitions[DefinitionIndex] != LoadedDefinitions[DefinitionIndex];
			if (bDefinitionHasChanged || bDefinitionEdittedAndNotValidated)
			{
				bReInit = true;
				break;
			}
		}
	}
	return bReInit;
}

bool FRigMapperOpHelper::InitializeRigMapping(const TArray<URigMapperDefinition*>& InDefinitions)
{
	RigMapperProcessor = FRigMapperProcessor(InDefinitions);
	LoadedDefinitions = InDefinitions;

	if (!RigMapperProcessor.IsValid())
	{
		return false;
	}

	// Cache a map of curve indices to bulk get current curve values for the input
	const TArray<FName>& InputNames = RigMapperProcessor.GetInputNames();
	InputCurveMappings.Empty();
	InputCurveMappings.Reserve(InputNames.Num());
	for (int32 InputIndex = 0; InputIndex < InputNames.Num(); InputIndex++)
	{
		InputCurveMappings.Add(InputNames[InputIndex], InputIndex);
	}

	// Cache a map of curve indices to bulk set the new curve values for the output
	const TArray<FName>& OutputNames = RigMapperProcessor.GetOutputNames();
	OutputCurveMappings.Empty();
	OutputCurveMappings.Reserve(OutputNames.Num());
	for (int32 OutputIndex = 0; OutputIndex < OutputNames.Num(); OutputIndex++)
	{
		const FName& CurveName = OutputNames[OutputIndex];

		int32 InputIndex = InputNames.Find(CurveName);
		OutputCurveMappings.Add(OutputNames[OutputIndex], OutputIndex, InputIndex);
	}

	return true;
}

void FRigMapperOpHelper::EvaluateRigMapping(const FBlendedCurve& InCurve, FBlendedCurve& OutCurve)
{
	if (!RigMapperProcessor.IsValid())
	{
		return;
	}

	// Retrieve inputs
	CachedInputValues.Reset(InputCurveMappings.Num());
	CachedInputValues.AddDefaulted(InputCurveMappings.Num());
	UE::Anim::FCurveUtils::BulkGet(InCurve, InputCurveMappings,
		[&Inputs = CachedInputValues](const FRigMapperCurveMapping& InBulkElement, const float InValue)
		{
			Inputs[InBulkElement.CurveIndex] = InValue;
		}
	);

	{
		// Evaluate frame
		RigMapperProcessor.EvaluateFrame(RigMapperProcessor.GetInputNames(), CachedInputValues, CachedOutputValues);
	}

	{
		// Set all output curves from the given output values.
		UE::Anim::FCurveUtils::BulkSet(OutCurve, OutputCurveMappings,
			[&Inputs = CachedInputValues, &Outputs = CachedOutputValues](const FRigMapperOutputCurveMapping& InBulkElement)
			{
				TOptional<float> Value = Outputs[InBulkElement.CurveIndex];

				// If the value was not set by the rig mapper default to 0
				return Value.Get(0.f);
			}
		);
	}
}

URigMapperDefinitionUserData* FRigMapperOpHelper::GetUserDataFromMesh(const USkeletalMesh* InMesh)
{
	if (!InMesh) return nullptr;

	UAssetUserData* AssetUserData = const_cast<USkeletalMesh*>(InMesh)->
		GetAssetUserDataOfClass(URigMapperDefinitionUserData::StaticClass());
	return Cast<URigMapperDefinitionUserData>(AssetUserData);
}

bool FIKRetargetRigMapperOp::Initialize(const FIKRetargetProcessor& InProcessor, const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton, const FIKRetargetOpBase* InParentOp, FIKRigLogger& Log)
{
	const FRetargetSkeleton& TargetSkeleton = InProcessor.GetSkeleton(ERetargetSourceOrTarget::Target);

	TArray<URigMapperDefinition*> DefinitionsToLoad = GetDefinitionsToLoad(TargetSkeleton.SkeletalMesh);
	bIsInitialized = Helper.InitializeRigMapping(DefinitionsToLoad);

	return bIsInitialized;
}

void FIKRetargetRigMapperOp::ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues, 
	FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const
{
	Helper.ProcessAnimSequenceCurves(InCurveMetaData, InCurveFrameValues, OutCurveMetaData, OutCurveFrameValues, Settings.bCopyAllSourceCurves);
}

void FIKRetargetRigMapperOp::AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent)
{
	if (!IsEnabled())
	{
		return;
	}

	SourceCurves.Empty();

	// get the source curves out of the source anim instance
	const UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}

	// Potential optimization/trade off: If we stored the curve results on the mesh component in non-editor scenarios, this would be
	// much faster (but take more memory). As it is, we need to translate the map stored on the anim instance.
	const TMap<FName, float>& AnimCurveList = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
	UE::Anim::FCurveUtils::BuildUnsorted(SourceCurves, AnimCurveList);

	// Need reinit if the definitions in use have changed.
	// If definitions were loaded from the SKM asset user data, they take priority and are the ones we will check against
	USkeletalMesh* TargetMesh = TargetMeshComponent.GetSkeletalMeshAsset();
	TArray<URigMapperDefinition*> DefinitionsToLoad = GetDefinitionsToLoad(TargetMesh);
	if (Helper.CheckReInit(DefinitionsToLoad))
	{
		bIsInitialized = Helper.InitializeRigMapping(DefinitionsToLoad);
	}
}

void FIKRetargetRigMapperOp::AnimGraphEvaluateAnyThread(FPoseContext& Output)
{
	if (!IsEnabled())
	{
		return;
	}

	FBlendedCurve& OutputCurves = Output.Curve;
	FBlendedCurve InputCurves;

	if (GetTakeInputCurvesFromSourceAnimInstance())
	{
		InputCurves.CopyFrom(SourceCurves);
		if (Settings.bCopyAllSourceCurves)
		{
			OutputCurves.CopyFrom(SourceCurves);
		}
	}
	else
	{
		InputCurves.CopyFrom(OutputCurves);
		// clear the outputs if needed
		if (!Settings.bCopyAllSourceCurves)
		{
			OutputCurves.ForEachElement([&OutputCurves](const UE::Anim::FCurveElement& InCurveElement)
				{
					OutputCurves.Set(InCurveElement.Name, 0.0f);
				});
		}
	}

	Helper.EvaluateRigMapping(InputCurves, OutputCurves);
}

bool FIKRetargetRigMapperOp::PostLoadSplitOp(TArray<FInstancedStruct>& OutNewOps)
{
	// Only split if there are multiple definitions in the old RigMapperOp
	if (Settings.Definitions.Num() <= 1)
	{
		// Migrate single definition and empty array
		if (Settings.Definitions.Num() == 1)
		{
			Settings.Definition = Settings.Definitions[0];
			Settings.Definitions.Empty();
		}
		return false; // Don't split, just migrate in place
	}

	// In case of multiple definitions create one op per definition
	const FName OriginalName = GetName();
	const bool bOriginalEnabled = IsEnabled();
	const FName ParentName = GetParentOpName();

	for (int32 DefinitionIndex = 0; DefinitionIndex < Settings.Definitions.Num(); DefinitionIndex++)
	{
		// Create new op instance
		FInstancedStruct NewOpStruct;
		NewOpStruct.InitializeAs(FIKRetargetRigMapperOp::StaticStruct());
		FIKRetargetRigMapperOp* NewOp = NewOpStruct.GetMutablePtr<FIKRetargetRigMapperOp>();

		if (NewOp)
		{
			// Copy settings
			NewOp->Settings.bCopyAllSourceCurves = Settings.bCopyAllSourceCurves;
			// Set single definition (new property) instead of deprecated array
			NewOp->Settings.Definition = Settings.Definitions[DefinitionIndex];

			// Copy enabled state
			NewOp->SetEnabled(bOriginalEnabled);

			// Set parent op
			if (!ParentName.IsNone())
			{
				NewOp->SetParentOpName(ParentName);
			}

			OutNewOps.Add(MoveTemp(NewOpStruct));
		}
	}

	return true; // Replace original op with new ops
}

TArray<URigMapperDefinition*> FIKRetargetRigMapperOp::GetDefinitionsToLoad(const USkeletalMesh* InTargetMesh)
{
	TArray<URigMapperDefinition*> DefinitionsToLoad;

	// If bOverrideFromUserDataDefinitions is true, get definitions from UserDataAsset
	if (Settings.bOverrideFromUserDataDefinitions)
	{
		URigMapperDefinitionUserData* UserData = FRigMapperOpHelper::GetUserDataFromMesh(InTargetMesh);
		if (UserData)
		{
			DefinitionsToLoad = UserData->Definitions;
		}
	}
	// Otherwise use single definition from the Op
	else if (Settings.Definition)
	{
		DefinitionsToLoad.Add(Settings.Definition);
	}
	return DefinitionsToLoad;
}

const UClass* FIKRetargetRigMapperOpSettings::GetControllerType() const
{
	return UIKRetargetRigMapperOpController::StaticClass();
}

void FIKRetargetRigMapperOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies all properties
	const TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetRigMapperOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetRigMapperUserDataOp::Initialize(const FIKRetargetProcessor& InProcessor, const FRetargetSkeleton& InSourceSkeleton, const FTargetSkeleton& InTargetSkeleton, const FIKRetargetOpBase* InParentOp, FIKRigLogger& Log)
{
	const FRetargetSkeleton& TargetSkeleton = InProcessor.GetSkeleton(ERetargetSourceOrTarget::Target);

	TArray<URigMapperDefinition*> DefinitionsToLoad = GetDefinitionsToLoad(TargetSkeleton.SkeletalMesh);
	bIsInitialized = Helper.InitializeRigMapping(DefinitionsToLoad);

	return bIsInitialized;
}

void FIKRetargetRigMapperUserDataOp::ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues, 
	FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const
{
	Helper.ProcessAnimSequenceCurves(InCurveMetaData, InCurveFrameValues, OutCurveMetaData, OutCurveFrameValues, Settings.bCopyAllSourceCurves);
}

void FIKRetargetRigMapperUserDataOp::AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent)
{
	if (!IsEnabled())
	{
		return;
	}

	SourceCurves.Empty();

	// get the source curves out of the source anim instance
	const UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}

	// Potential optimization/trade off: If we stored the curve results on the mesh component in non-editor scenarios, this would be
	// much faster (but take more memory). As it is, we need to translate the map stored on the anim instance.
	const TMap<FName, float>& AnimCurveList = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
	UE::Anim::FCurveUtils::BuildUnsorted(SourceCurves, AnimCurveList);

	// Need reinit if the definitions in use have changed.
	// If definitions were loaded from the SKM asset user data, they take priority and are the ones we will check against
	USkeletalMesh* TargetMesh = TargetMeshComponent.GetSkeletalMeshAsset();
	TArray<URigMapperDefinition*> DefinitionsToLoad = GetDefinitionsToLoad(TargetMesh);
	if (Helper.CheckReInit(DefinitionsToLoad))
	{
		bIsInitialized = Helper.InitializeRigMapping(DefinitionsToLoad);
	}
}

void FIKRetargetRigMapperUserDataOp::AnimGraphEvaluateAnyThread(FPoseContext& Output)
{
	if (!IsEnabled())
	{
		return;
	}

	FBlendedCurve& OutputCurves = Output.Curve;
	FBlendedCurve InputCurves;

	if (GetTakeInputCurvesFromSourceAnimInstance())
	{
		InputCurves.CopyFrom(SourceCurves);
		if (Settings.bCopyAllSourceCurves)
		{
			OutputCurves.CopyFrom(SourceCurves);
		}
	}
	else
	{
		InputCurves.CopyFrom(OutputCurves);
		// clear the outputs if needed
		if (!Settings.bCopyAllSourceCurves)
		{
			OutputCurves.ForEachElement([&OutputCurves](const UE::Anim::FCurveElement& InCurveElement)
				{
					OutputCurves.Set(InCurveElement.Name, 0.0f);
				});
		}
	}

	Helper.EvaluateRigMapping(InputCurves, OutputCurves);
}

TArray<URigMapperDefinition*> FIKRetargetRigMapperUserDataOp::GetDefinitionsToLoad(const USkeletalMesh* InTargetMesh)
{
	TArray<URigMapperDefinition*> DefinitionsToLoad;

	URigMapperDefinitionUserData* UserData = FRigMapperOpHelper::GetUserDataFromMesh(InTargetMesh);
	if (UserData)
	{
		DefinitionsToLoad = UserData->Definitions;
	}

	return DefinitionsToLoad;
}

const UClass* FIKRetargetRigMapperUserDataOpSettings::GetControllerType() const
{
	return UIKRetargetRigMapperUserDataOpController::StaticClass();
}

void FIKRetargetRigMapperUserDataOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies all properties
	const TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetRigMapperUserDataOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

void UIKRetargetRigMapperOpController::SetSettings(FIKRetargetRigMapperOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

FIKRetargetRigMapperOpSettings UIKRetargetRigMapperOpController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetRigMapperOpSettings*>(OpSettingsToControl);
}

void UIKRetargetRigMapperUserDataOpController::SetSettings(FIKRetargetRigMapperUserDataOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

FIKRetargetRigMapperUserDataOpSettings UIKRetargetRigMapperUserDataOpController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetRigMapperUserDataOpSettings*>(OpSettingsToControl);
}
