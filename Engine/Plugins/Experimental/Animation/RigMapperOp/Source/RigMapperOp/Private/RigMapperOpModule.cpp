// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigMapperOpModule.h"

#define LOCTEXT_NAMESPACE "RigMapperOpModule"

void FRigMapperOpModule::StartupModule()
{
}

void FRigMapperOpModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FRigMapperOpModule, RigMapperOp)