// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/RetargetCurvesOp.h"
#include "Retargeter/RetargetOps/CurveRemapOp.h"
#include "Animation/AnimCurveTypes.h"
#include "RigMapperProcessor.h"

#include "RigMapperOp.generated.h"

#define UE_API RIGMAPPEROP_API

#define LOCTEXT_NAMESPACE "RigMapperOp"

class URigMapperDefinitionUserData;
class URigMapperDefinition;
class USkeletalMesh;
class USkeletalMeshComponent;

USTRUCT(BlueprintType)
struct FIKRetargetRigMapperOpSettingsBase : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	// Whether to copy all curves over to the target animation instance
	// NOTE: This setting also applies when exporting retargeted animations.
	// True: all source curves are copied to the target animation instance/asset
	// False: only remapped curves are copied on the target animation instance/asset
	// In general, we should set this to true if the source and target rig are the same,
	// and the RigMapper covers only a subset of the controls, but false otherwise.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Copy Curves")
	bool bCopyAllSourceCurves = false;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Single Definition RigMapper Settings"))
struct FIKRetargetRigMapperOpSettings : public FIKRetargetRigMapperOpSettingsBase
{
	GENERATED_BODY();

	// User defined RigMapper definition
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RigMapper)
	TObjectPtr<URigMapperDefinition> Definition;

	// Whether should RigMapper definition be overridden with the array of definitions from target skeletal mesh UserData
	// NOTE: Should not be used if there are multiple single RigMapper ops in the sequence
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RigMapper)
	bool bOverrideFromUserDataDefinitions = false;

	// TODO: Remove in UE 5.8
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Definition instead"))
	TArray<TObjectPtr<URigMapperDefinition>> Definitions;

	UE_API virtual const UClass* GetControllerType() const override;
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "UserData RigMapper Settings"))
struct FIKRetargetRigMapperUserDataOpSettings : public FIKRetargetRigMapperOpSettingsBase
{
	GENERATED_BODY();

	UE_API virtual const UClass* GetControllerType() const override;
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

// A helper struct containing business logic common for all RigMapper ops
struct RIGMAPPEROP_API FRigMapperOpHelper
{
public:
	FRigMapperOpHelper() = default;

	bool InitializeRigMapping(const TArray<URigMapperDefinition*>& InDefinitions);

	void ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
		FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues, bool bCopyAllSourceCurves) const;

	void EvaluateRigMapping(const FBlendedCurve& InCurve, FBlendedCurve& OutCurve);

	bool CheckReInit(const TArray<URigMapperDefinition*>& InCurrentDefinitions);

	bool IsValid() const { return RigMapperProcessor.IsValid(); }

	static URigMapperDefinitionUserData* GetUserDataFromMesh(const USkeletalMesh* InMesh);

private:

	// The definitions that we have loaded. Cached to check against changes and reinit if need be
	TArray<TObjectPtr<URigMapperDefinition>> LoadedDefinitions;

	// The processor to evaluate the rig mapping
	FRigMapperProcessor RigMapperProcessor;

	// The cached input values passed to the rig mapper processor to avoid reallocations
	FRigMapperProcessor::FPoseValues CachedInputValues;

	// The cached output values passed to the rig mapper processor to avoid reallocations
	FRigMapperProcessor::FPoseValues CachedOutputValues;

	// Base curve mapping for bulk get/set of the linked pose curves
	struct FRigMapperCurveMapping
	{
		FRigMapperCurveMapping() = default;

		FRigMapperCurveMapping(FName InName, int32 InCurveIndex)
			: Name(InName)
			, CurveIndex(InCurveIndex)
		{
		}

		FName Name = NAME_None;
		int32 CurveIndex = INDEX_NONE;
	};

	// Curve mapping for bulk set of the curves with a mapping to the matching input
	struct FRigMapperOutputCurveMapping : FRigMapperCurveMapping
	{
		FRigMapperOutputCurveMapping() = default;

		FRigMapperOutputCurveMapping(FName InName, int32 InOutputCurveIndex, int32 InInputCurveIndex)
			: FRigMapperCurveMapping(InName, InOutputCurveIndex)
			, InputCurveIndex(InInputCurveIndex)
		{
		}

		int32 InputCurveIndex = INDEX_NONE;
	};

	// Input curve mapping for bulk get of the curve values
	using FInputCurveMappings = UE::Anim::TNamedValueArray<FDefaultAllocator, FRigMapperCurveMapping>;
	FInputCurveMappings InputCurveMappings;

	// Output curve mapping for the bulk get of the curve values
	using FOutputCurveMappings = UE::Anim::TNamedValueArray<FDefaultAllocator, FRigMapperOutputCurveMapping>;
	FOutputCurveMappings OutputCurveMappings;
};

// Single definition RigMapper op with an option to use UserData definitions array
USTRUCT(BlueprintType, meta = (DisplayName = "Single RigMapper"))
struct FIKRetargetRigMapperOp : public FIKRetargetCurvesOpBase
{
	GENERATED_BODY()
	
public:	
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;

	// NOTE: RigMapper ops do not do anything in Run().
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override {};

	virtual FIKRetargetOpSettingsBase* GetSettings() override
	{
		return &Settings;
	};

	virtual void SetSettings(const FIKRetargetOpSettingsBase* InSettings) override
	{
		Settings = *reinterpret_cast<const FIKRetargetRigMapperOpSettings*>(InSettings);
	};
	
	virtual const UScriptStruct* GetSettingsType() const override
	{
		return FIKRetargetRigMapperOpSettings::StaticStruct();
	}

	virtual const UScriptStruct* GetParentOpType() const override
	{
		return FIKRetargetCurveRemapOp::StaticStruct();
	}
	
	virtual const UScriptStruct* GetType() const override
	{
		return FIKRetargetRigMapperOp::StaticStruct();
	}

	virtual bool IsAbstract() const override { return false; };

	virtual bool IsSingleton() const override { return false; }

	virtual void ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
		FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const override;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual void AnimGraphEvaluateAnyThread(FPoseContext& Output) override;

	bool PostLoadSplitOp(TArray<FInstancedStruct>& OutNewOps);

	UPROPERTY()
	FIKRetargetRigMapperOpSettings Settings;

protected:
	TArray<URigMapperDefinition*> GetDefinitionsToLoad(const USkeletalMesh* InTargetMesh);

	// A helper struct containing business logic common for all RigMapper ops
	FRigMapperOpHelper Helper;
};

/* The blueprint/python API for editing a RigMapper Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetRigMapperOpController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetRigMapperOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetRigMapperOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetRigMapperOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetRigMapperOpSettings InSettings);
};

// UserData definitions ONLY RigMapper op
USTRUCT(BlueprintType, meta = (DisplayName = "UserData RigMapper"))
struct FIKRetargetRigMapperUserDataOp : public FIKRetargetCurvesOpBase
{
	GENERATED_BODY()

public:
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;

	// NOTE: RigMapper ops do not do anything in Run().
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override {
	};

	virtual FIKRetargetOpSettingsBase* GetSettings() override
	{
		return &Settings;
	};

	virtual void SetSettings(const FIKRetargetOpSettingsBase* InSettings) override
	{
		Settings = *reinterpret_cast<const FIKRetargetRigMapperUserDataOpSettings*>(InSettings);
	};

	virtual const UScriptStruct* GetSettingsType() const override
	{
		return FIKRetargetRigMapperUserDataOpSettings::StaticStruct();
	}

	virtual const UScriptStruct* GetParentOpType() const override
	{
		return FIKRetargetCurveRemapOp::StaticStruct();
	}

	virtual const UScriptStruct* GetType() const override
	{
		return FIKRetargetRigMapperUserDataOp::StaticStruct();
	}

	virtual bool IsAbstract() const override { return false; };

	virtual bool IsSingleton() const override { return false; }

	virtual void ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
		FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual void AnimGraphEvaluateAnyThread(FPoseContext& Output) override;

	UPROPERTY()
	FIKRetargetRigMapperUserDataOpSettings Settings;

protected:
	TArray<URigMapperDefinition*> GetDefinitionsToLoad(const USkeletalMesh* InTargetMesh);

	// A helper struct containing business logic common for all RigMapper ops
	FRigMapperOpHelper Helper;
};

/* The blueprint/python API for editing a RigMapper UserData Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetRigMapperUserDataOpController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()

public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetRigMapperUserDataOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetRigMapperUserDataOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetRigMapperUserDataOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetRigMapperUserDataOpSettings InSettings);

};

#undef LOCTEXT_NAMESPACE

#undef UE_API
