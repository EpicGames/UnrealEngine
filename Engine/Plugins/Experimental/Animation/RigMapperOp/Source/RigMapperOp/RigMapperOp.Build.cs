// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO; // for Path

namespace UnrealBuildTool.Rules
{
	public class RigMapperOp : ModuleRules
	{
		public RigMapperOp(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PublicIncludePaths.AddRange(
				new string[] {
					// ... add public include paths required here ...
				}
				);


			PrivateIncludePaths.AddRange(
				new string[] {
					// ... add other private include paths required here ...
				}
				);


			PublicDependencyModuleNames.AddRange(
				new string[]
				{
				}
				);


			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				"Core",
				"CoreUObject",
				"Engine",
				"AnimationCore",
				"IKRig",
				"RigMapper"
				}
				);


			DynamicallyLoadedModuleNames.AddRange(
				new string[]
				{
					// ... add any modules that your module loads dynamically here ...
				}
				);
		}
	}

}